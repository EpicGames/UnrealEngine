// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"

#include "HandlerComponentFactory.generated.h"

class HandlerComponent;

/**
 * A UObject alternative for loading HandlerComponents without strict module dependency
 */
UCLASS(abstract, MinimalAPI)
class UHandlerComponentFactory : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	virtual TSharedPtr<HandlerComponent> CreateComponentInstance(FString& Options)
		PURE_VIRTUAL(UHandlerComponentFactory::CreateComponentInstance, return nullptr;);
};
