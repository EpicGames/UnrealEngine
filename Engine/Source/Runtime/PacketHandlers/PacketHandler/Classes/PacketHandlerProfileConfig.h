// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "PacketHandlerProfileConfig.generated.h"

/**
 * PacketHandler Configuration Profiles based on each NetDriver
 */
UCLASS(config=Engine, PerObjectConfig)
class UPacketHandlerProfileConfig : public UObject
{
	GENERATED_UCLASS_BODY()

public:

	UPROPERTY(config)
	TArray<FString> Components;
};
