// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class PacketHandler : ModuleRules
{
    public PacketHandler(ReadOnlyTargetRules Target) : base(Target)
    {
        PublicDependencyModuleNames.AddRange
		(
			new string[]
			{
				"Core",
				"CoreUObject",
				"NetCore",
				"ReliabilityHandlerComponent",
				"Sockets",
			}
        );

        CircularlyReferencedDependentModules.Add("ReliabilityHandlerComponent");

		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
	}
}
