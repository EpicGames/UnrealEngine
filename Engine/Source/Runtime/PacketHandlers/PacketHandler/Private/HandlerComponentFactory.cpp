// Copyright Epic Games, Inc. All Rights Reserved.

#include "HandlerComponentFactory.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HandlerComponentFactory)


/**
 * UHandlerComponentFactory
 */
UHandlerComponentFactory::UHandlerComponentFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

