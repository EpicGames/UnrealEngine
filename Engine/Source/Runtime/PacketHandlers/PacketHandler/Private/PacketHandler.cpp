// Copyright Epic Games, Inc. All Rights Reserved.

#include "PacketHandler.h"
#include "Net/Core/Misc/PacketAudit.h"
#include "EncryptionComponent.h"

#include "Misc/ConfigCacheIni.h"
#include "Modules/ModuleManager.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Package.h"
#include "HAL/ConsoleManager.h"

#include "Net/Core/Misc/DDoSDetection.h"
#include "HandlerComponentFactory.h"
#include "ReliabilityHandlerComponent.h"
#include "PacketHandlerProfileConfig.h"

#include "SocketSubsystem.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/StringBuilder.h"
#include "Misc/TransactionallySafeRWLock.h"

// @todo #JohnB: There is quite a lot of inefficient copying of packet data going on.
//					Redo the whole packet parsing/modification pipeline.

IMPLEMENT_MODULE(FPacketHandlerComponentModuleInterface, PacketHandler);

DEFINE_LOG_CATEGORY(PacketHandlerLog);

DECLARE_CYCLE_STAT(TEXT("PacketHandler Incoming_Internal"), Stat_PacketHandler_Incoming_Internal, STATGROUP_Net);
DECLARE_CYCLE_STAT(TEXT("PacketHandler Outgoing_Internal"), Stat_PacketHandler_Outgoing_Internal, STATGROUP_Net);

// CVars

#if !UE_BUILD_SHIPPING
int32 GPacketHandlerCRCDump = 0;

FAutoConsoleVariableRef CVarNetPacketHandlerCRCDump(
	TEXT("net.PacketHandlerCRCDump"),
	GPacketHandlerCRCDump,
	TEXT("Enables or disables dumping of packet CRC's for every HandlerComponent, Incoming and Outgoing, for debugging."));

static FTransactionallySafeRWLock PacketHandlerTimeguardLock;
static int32 GPacketHandlerTimeguardLimit = 20;
static float GPacketHandlerTimeguardThresholdMS = 0.0f;
static UE_AUTORTFM_DECLARE_THREAD_LOCAL_VAR(bool, GPacketHandlerDiscardTimeguardMeasurement);

static FAutoConsoleVariableRef CVarNetPacketHandlerTimeguardThresholdMS(
	TEXT("net.PacketHandlerTimeguardThresholdMS"),
	GPacketHandlerTimeguardThresholdMS,
	TEXT("Threshold in milliseconds for the HandlerComponent timeguard, Incoming and Outgoing."),
	ECVF_Default);
static FAutoConsoleVariableRef CVarNetPacketHandlerTimeguardLimit(
	TEXT("net.PacketHandlerTimeguardLimit"),
	GPacketHandlerTimeguardLimit,
	TEXT("Sets the maximum number of HandlerComponent timeguard logs.\n"),
	ECVF_Default
);

namespace UE
{
	namespace Net
	{
		// GPacketHandlerDiscardTimeguardMeasurement is unique per-thread, but accessible from other
		// compilation units, so it's exposed here.
		void SetDiscardTimeguardMeasurement(const bool bShouldDiscard)
		{
			GPacketHandlerDiscardTimeguardMeasurement = bShouldDiscard;
		}

		bool GetDiscardTimeguardMeasurement()
		{
			return GPacketHandlerDiscardTimeguardMeasurement;
		}

		// GPacketHandlerTimeguardLimit can be accessed across threads to limit the total number of time guard warnings
		// In cases where multiple threads go above the threshold simultaneously, it's possible to go above GPacketHandlerTimeguardLimit
		// but after that frame no more warnings will be sent.
		int32 GetTimeguardLimit()
		{
			UE::TReadScopeLock Lock(PacketHandlerTimeguardLock);
			return GPacketHandlerTimeguardLimit;
		}

		void DecrementTimeguardLimit()
		{
			UE::TWriteScopeLock Lock(PacketHandlerTimeguardLock);
			GPacketHandlerTimeguardLimit--;
		}
	}
}

// Lightweight time guard. Note: Threshold of 0 disables the timeguard
#define NET_LIGHTWEIGHT_TIME_GUARD_BEGIN( Name, ThresholdMS ) \
	double UE_JOIN(__TimeGuard_ThresholdMS_, Name) = ThresholdMS; \
	uint64 UE_JOIN(__TimeGuard_StartCycles_, Name) = ( ThresholdMS > 0.0f && UE::Net::GetTimeguardLimit() > 0 ) ? FPlatformTime::Cycles64() : 0; \
	UE::Net::SetDiscardTimeguardMeasurement(false);

#define NET_LIGHTWEIGHT_TIME_GUARD_END( Name, NameStringCode ) \
	if ( UE_JOIN(__TimeGuard_ThresholdMS_, Name) > 0.0f && UE::Net::GetTimeguardLimit() > 0 && !UE::Net::GetDiscardTimeguardMeasurement() ) \
	{\
		double UE_JOIN(__TimeGuard_MSElapsed_,Name) = FPlatformTime::ToMilliseconds64( FPlatformTime::Cycles64() - UE_JOIN(__TimeGuard_StartCycles_,Name) ); \
		if ( UE_JOIN(__TimeGuard_MSElapsed_,Name) > UE_JOIN(__TimeGuard_ThresholdMS_, Name) ) \
		{ \
			FString ReportName = NameStringCode; \
			UE_LOGF(PacketHandlerLog, Warning, "PacketHandler: %ls - %ls took %.2fms!", TEXT(#Name), *ReportName, UE_JOIN(__TimeGuard_MSElapsed_,Name)); \
			UE::Net::DecrementTimeguardLimit(); \
		} \
	}
#else // UE_BUILD_SHIPPING
  #define NET_LIGHTWEIGHT_TIME_GUARD_BEGIN( Name, ThresholdMS )
  #define NET_LIGHTWEIGHT_TIME_GUARD_END( Name, NameStringCode )
#endif

template<typename OutType, typename InType>
OutType IntCastLog(InType In)
{
	bool bFitsIn = IntFitsIn<OutType, InType>(In);
	ensureMsgf(bFitsIn, TEXT("PacketHandler: Loss of data caused by truncated cast"));
	UE_CLOGF(!bFitsIn, PacketHandlerLog, Warning, "PacketHandler: Loss of data caused by truncated cast");
	return static_cast<OutType>(In);
}

/**
 * BufferedPacket
 */

BufferedPacket::~BufferedPacket()
{
	delete [] Data;
}

/**
 * PacketHandler
 */

PacketHandler::PacketHandler(FDDoSDetection* InDDoS/*=nullptr*/)
	: Mode(UE::Handler::Mode::Client)
	, State(UE::Handler::State::Uninitialized)
	, bRawSend(false)
	, bBeganHandshaking(false)
	, bConnectionlessHandler(false)
	, MaxPacketBits(0)
	, DDoS(InDDoS)
	, LowLevelSendDel()
	, HandshakeCompleteDel()
	, OutgoingPacket(MAX_PACKET_SIZE * 8)
	, IncomingPacket()
	, HandlerComponents()
	, BufferedPackets()
	, QueuedPackets()
	, QueuedRawPackets()
	, QueuedHandlerPackets()
	, BufferedConnectionlessPackets()
	, QueuedConnectionlessPackets()
	, ReliabilityComponent(nullptr)
	, Provider()
	, Aggregator()
{
	OutgoingPacket.SetAllowResize(true);
	OutgoingPacket.AllowAppend(true);
}

void PacketHandler::Tick(float DeltaTime)
{
	for (const TSharedPtr<HandlerComponent>& Component : HandlerComponents)
	{
		if (Component.IsValid())
		{
			Component->Tick(DeltaTime);
		}
	}

	// Send off any queued handler packets
	BufferedPacket* QueuedPacket = nullptr;

	while (QueuedHandlerPackets.Dequeue(QueuedPacket))
	{
		check(QueuedPacket->FromComponent != nullptr);

		FBitWriter OutPacket;

		OutPacket.SerializeBits(QueuedPacket->Data, QueuedPacket->CountBits);

		SendHandlerPacket(QueuedPacket->FromComponent, OutPacket, QueuedPacket->Traits);
	}
}

FPacketHandlerAddComponentByNameDelegate& PacketHandler::GetAddComponentByNameDelegate()
{
	static FPacketHandlerAddComponentByNameDelegate AddComponentByNameDelegate;
	return AddComponentByNameDelegate;
}

FPacketHandlerAddComponentDelegate& PacketHandler::GetAddComponentDelegate()
{
	static FPacketHandlerAddComponentDelegate AddComponentDelegate;
	return AddComponentDelegate;
}

void PacketHandler::Initialize(UE::Handler::Mode InMode, uint32 InMaxPacketBits, bool bConnectionlessOnly/*=false*/,
								TSharedPtr<IAnalyticsProvider> InProvider/*=nullptr*/, FDDoSDetection* InDDoS/*=nullptr*/, FName InDriverProfile/*=NAME_None*/)
{
	Mode = InMode;
	MaxPacketBits = InMaxPacketBits;
	DDoS = InDDoS;

	bConnectionlessHandler = bConnectionlessOnly;

	// Only UNetConnection's will load the .ini components, for now.
	if (!bConnectionlessHandler)
	{
		TArray<FString> Components;
		FString DriverProfileCategory = FString::Printf(TEXT("%s PacketHandlerProfileConfig"), *InDriverProfile.GetPlainNameString());
		GConfig->GetArray(*DriverProfileCategory, TEXT("Components"), Components, GEngineIni);
		
		// If we didn't get any matches, push in the regular components.
		if (Components.Num() == 0)
		{
			GConfig->GetArray(TEXT("PacketHandlerComponents"), TEXT("Components"), Components, GEngineIni);
		}

		// Users of this delegate can add components to the list by name and if necessary reorder them
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		GetAddComponentByNameDelegate().ExecuteIfBound(Components);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		for (const FString& CurComponent : Components)
		{
			AddHandler(CurComponent, true);
		}

		// all of code below till end of scope can go away once deprecated GetAddComponentDelegate method is removed
		PRAGMA_DISABLE_DEPRECATION_WARNINGS

		// Users of this delegate can supply constructed additional components to be added after the named components
		TArray<TSharedPtr<HandlerComponent>> AdditionalComponents;
		GetAddComponentDelegate().ExecuteIfBound(AdditionalComponents);

		for (TSharedPtr<HandlerComponent> AdditionalComponent : AdditionalComponents)
		{
			AddHandler(AdditionalComponent, true);
		}

		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	// Add encryption component, if configured.
	FString EncryptionComponentName;
	if (GConfig->GetString(TEXT("PacketHandlerComponents"), TEXT("EncryptionComponent"), EncryptionComponentName, GEngineIni) && !EncryptionComponentName.IsEmpty())
	{
		static IConsoleVariable* const AllowEncryptionCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("net.AllowEncryption"));
		if (AllowEncryptionCVar == nullptr || AllowEncryptionCVar->GetInt() != 0)
		{
			EncryptionComponent = StaticCastSharedPtr<FEncryptionComponent>(AddHandler(EncryptionComponentName, true));
		}
		else
		{
			UE_LOGF(PacketHandlerLog, Warning, "PacketHandler encryption component is configured as %ls, but it won't be used because the cvar net.AllowEncryption is false.", *EncryptionComponentName);
		}
	}

	bool bEnableReliability = false;

	GConfig->GetBool(TEXT("PacketHandlerComponents"), TEXT("bEnableReliability"), bEnableReliability, GEngineIni);

	if (bEnableReliability && !ReliabilityComponent.IsValid())
	{
		UE_LOGF(PacketHandlerLog, Warning, "bEnableReliability for PacketHandlerComponents is deprecated. For fully-reliable data, use reliable RPCs or a separate connection with a reliable protocol.");

PRAGMA_DISABLE_DEPRECATION_WARNINGS
		TSharedPtr<HandlerComponent> NewComponent = MakeShareable(new ReliabilityHandlerComponent);
		ReliabilityComponent = StaticCastSharedPtr<ReliabilityHandlerComponent>(NewComponent);
		AddHandler(NewComponent, true);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

void PacketHandler::InitializeDelegates(FPacketHandlerLowLevelSendTraits InLowLevelSendDel,
										FPacketHandlerNotifyAddHandler InAddHandlerDel/*=FPacketHandlerNotifyAddHandler()*/)
{
	LowLevelSendDel = InLowLevelSendDel;
	AddHandlerDel = InAddHandlerDel;
}

void PacketHandler::InitFaultRecovery(UE::Net::FNetConnectionFaultRecoveryBase* InFaultRecovery)
{
	for (TSharedPtr<HandlerComponent>& CurComponent : HandlerComponents)
	{
		CurComponent->InitFaultRecovery(InFaultRecovery);
	}
}

void PacketHandler::NotifyAnalyticsProvider(TSharedPtr<IAnalyticsProvider> InProvider, TSharedPtr<FNetAnalyticsAggregator> InAggregator)
{
	Provider = InProvider;
	Aggregator = InAggregator;

	if (State != UE::Handler::State::Uninitialized)
	{
		for (const TSharedPtr<HandlerComponent>& CurComponent : HandlerComponents)
		{
			if (CurComponent->IsInitialized())
			{
				CurComponent->NotifyAnalyticsProvider();
			}
		}
	}
}

void PacketHandler::InitializeComponents()
{
	if (State == UE::Handler::State::Uninitialized)
	{
		if (HandlerComponents.Num() > 0)
		{
			SetState(UE::Handler::State::InitializingComponents);
		}
		else
		{
			HandlerInitialized();
		}
	}

	// Trigger delayed-initialization for HandlerComponents
	for (TSharedPtr<HandlerComponent>& Component : HandlerComponents)
	{
		if (Component.IsValid() && !Component->IsInitialized())
		{
			Component->Initialize();
			Component->NotifyAnalyticsProvider();
		}
	}

	// Called early, to ensure that all handlers report a valid reserved packet bits value (triggers an assert if not)
	GetTotalReservedPacketBits();
}

void PacketHandler::BeginHandshaking(FPacketHandlerHandshakeComplete InHandshakeDel/*=FPacketHandlerHandshakeComplete()*/)
{
	check(!bBeganHandshaking);

	bBeganHandshaking = true;

	HandshakeCompleteDel = InHandshakeDel;

	for (int32 i=HandlerComponents.Num() - 1; i>=0; --i)
	{
		HandlerComponent& CurComponent = *HandlerComponents[i];

		if (CurComponent.RequiresHandshake() && !CurComponent.IsInitialized())
		{
			CurComponent.NotifyHandshakeBegin();
			break;
		}
	}
}

void PacketHandler::AddHandler(TSharedPtr<HandlerComponent>& NewHandler, bool bDeferInitialize/*=false*/)
{
	// This is never valid. Can end up silently changing maximum allow packet size, which could cause failure to send packets.
	if (State != UE::Handler::State::Uninitialized)
	{
		LowLevelFatalError(TEXT("Handler added during runtime."));
		return;
	}

	// This should always be fatal, as an unexpectedly missing handler, may break net compatibility with the remote server/client
	if (!NewHandler.IsValid())
	{
		LowLevelFatalError(TEXT("Failed to add handler - invalid instance."));
		return;
	}

	// Warn if a component already exists with the same name.
	const bool bNameAlreadyExists = HandlerComponents.ContainsByPredicate([NewHandler](const TSharedPtr<HandlerComponent>& Component)
	{
		return Component->GetName() == NewHandler->GetName();
	});

	if (bNameAlreadyExists)
	{
		UE_LOGF(PacketHandlerLog, Warning, "Packet handler already contains a component with name %ls.", *NewHandler->GetName().ToString());
		return;
	}

	HandlerComponents.Add(NewHandler);
	NewHandler->Handler = this;

	AddHandlerDel.ExecuteIfBound(NewHandler);

	if (!bDeferInitialize)
	{
		NewHandler->Initialize();
	}
}

TSharedPtr<HandlerComponent> PacketHandler::AddHandler(const FString& ComponentStr, bool bDeferInitialize/*=false*/)
{
	TSharedPtr<HandlerComponent> ReturnVal = nullptr;

	if (!ComponentStr.IsEmpty())
	{
		FString ComponentName;
		FString ComponentOptions;

		for (int32 i=0; i<ComponentStr.Len(); i++)
		{
			TCHAR c = ComponentStr[i];

			// Parsing Options
			if (c == '(')
			{
				// Skip '('
				++i;

				// Parse until end of options
				for (; i<ComponentStr.Len(); i++)
				{
					c = ComponentStr[i];

					// End of options
					if (c == ')')
					{
						break;
					}
					// Append char to options
					else
					{
						ComponentOptions.AppendChar(c);
					}
				}
			}
			// Append char to component name if not whitespace
			else if (c != ' ')
			{
				ComponentName.AppendChar(c);
			}
		}

		if (ComponentName != TEXT("ReliabilityHandlerComponent"))
		{
			int32 FactoryComponentDelim = ComponentName.Find(TEXT("."));

			if (FactoryComponentDelim != INDEX_NONE)
			{
				// Every HandlerComponentFactory type has one instance, loaded as a named singleton
				FString SingletonName = ComponentName.Mid(FactoryComponentDelim + 1) + TEXT("_Singleton");
				UHandlerComponentFactory* Factory = FindFirstObject<UHandlerComponentFactory>(*SingletonName, EFindFirstObjectOptions::NativeFirst | EFindFirstObjectOptions::EnsureIfAmbiguous);

				if (Factory == nullptr)
				{
					UClass* FactoryClass = StaticLoadClass(UHandlerComponentFactory::StaticClass(), nullptr, *ComponentName);

					if (FactoryClass != nullptr)
					{
						Factory = NewObject<UHandlerComponentFactory>(GetTransientPackage(), FactoryClass, *SingletonName);
					}
				}


				if (Factory != nullptr)
				{
					ReturnVal = Factory->CreateComponentInstance(ComponentOptions);
				}
				else
				{
					UE_LOGF(PacketHandlerLog, Warning, "Unable to load HandlerComponent factory: %ls", *ComponentName);
				}
			}
			// @todo #JohnB: Deprecate non-factory components eventually
			else
			{
				FPacketHandlerComponentModuleInterface* PacketHandlerInterface = FModuleManager::Get().LoadModulePtr<FPacketHandlerComponentModuleInterface>(FName(*ComponentName));

				if (PacketHandlerInterface != nullptr)
				{
					ReturnVal = PacketHandlerInterface->CreateComponentInstance(ComponentOptions);
				}

				if (!ReturnVal.IsValid())
				{
					UE_LOGF(PacketHandlerLog, Warning, "Unable to Load Module: %ls", *ComponentName);
				}
			}


			if (ReturnVal.IsValid())
			{
				UE_LOGF(PacketHandlerLog, Log, "Loaded PacketHandler component: %ls (%ls)", *ComponentName,
						*ComponentOptions);

				AddHandler(ReturnVal, bDeferInitialize);
			}
		}
		else
		{
			UE_LOGF(PacketHandlerLog, Warning, "PacketHandlerComponent 'ReliabilityHandlerComponent' is internal-only.");
		}
	}

	return ReturnVal;
}

void PacketHandler::IncomingHigh(FBitReader& Reader)
{
	// @todo #JohnB
}

void PacketHandler::OutgoingHigh(FBitWriter& Writer)
{
	// @todo #JohnB
}

TSharedPtr<FEncryptionComponent> PacketHandler::GetEncryptionComponent()
{
	return EncryptionComponent;
}

TSharedPtr<HandlerComponent> PacketHandler::GetComponentByName(FName ComponentName) const
{
	for (const TSharedPtr<HandlerComponent>& Component : HandlerComponents)
	{
		if (Component.IsValid() && Component->GetName() == ComponentName)
		{
			return Component;
		}
	}

	return nullptr;
}

void PacketHandler::CountBytes(FArchive& Ar) const
{
	Ar.CountBytes(sizeof(*this), sizeof(*this));
	OutgoingPacket.CountMemory(Ar);
	IncomingPacket.CountMemory(Ar);

	HandlerComponents.CountBytes(Ar);
	for (const TSharedPtr<HandlerComponent>& Component : HandlerComponents)
	{
		if (HandlerComponent const * const LocalComponent = Component.Get())
		{
			LocalComponent->CountBytes(Ar);
		}
	}

	// Don't handle EncryptionComponent, as it should be in our components array.

	BufferedPackets.CountBytes(Ar);
	for (BufferedPacket const * const LocalPacket : BufferedPackets)
	{
		if (LocalPacket)
		{
			LocalPacket->CountBytes(Ar);
		}
	}

	// Unfortunately, there's currently no way to safely calculate memory usage for TQueues.
	// so QueuedPackets, QueuedRawPackets, QueuedHandlerPackets, and QueuedConnectionlessPackets
	// can't be tracked without a rework.

	BufferedConnectionlessPackets.CountBytes(Ar);
	for (BufferedPacket const * const LocalPacket : BufferedConnectionlessPackets)
	{
		if (LocalPacket)
		{
			LocalPacket->CountBytes(Ar);
		}
	}

	// Don't handle ReliabilityComponent, since it should be in our components array.
	// Don't track AnalyticsProvider as that should be handled elsewhere.
}

void HandlerComponent::CountBytes(FArchive& Ar) const
{
	Ar.CountBytes(sizeof(*this), sizeof(*this));
}

EIncomingResult PacketHandler::Incoming_Internal(FReceivedPacketView& PacketView)
{
	SCOPE_CYCLE_COUNTER(Stat_PacketHandler_Incoming_Internal);

	EIncomingResult ReturnVal = EIncomingResult::Success;
	FPacketDataView& DataView = PacketView.DataView;
	int32 CountBits = DataView.NumBits();

#if !UE_BUILD_SHIPPING
	uint32 SocketCRC = 0;

	if (UNLIKELY(!!GPacketHandlerCRCDump))
	{
		SocketCRC = FCrc::MemCrc32(DataView.GetData(), DataView.NumBytes());
	}
#endif

	if (HandlerComponents.Num() > 0)
	{
		const uint8* DataPtr = DataView.GetData();
		uint8 LastByte = (UNLIKELY(DataPtr == nullptr)) ? 0 : DataPtr[DataView.NumBytes() - 1];

		if (LastByte != 0)
		{
			CountBits--;

			while (!(LastByte & 0x80))
			{
				LastByte *= 2;
				CountBits--;
			}
		}
		else
		{
			PacketView.DataView = {nullptr, 0, ECountUnits::Bits};
			ReturnVal = EIncomingResult::Error;

#if !UE_BUILD_SHIPPING
			UE_CLOGF((DDoS == nullptr || !DDoS->CheckLogRestrictions()), PacketHandlerLog, Error,
					"PacketHandler parsing packet with zero's in last byte.");
#endif
		}
	}

#if !UE_BUILD_SHIPPING
	struct FHandlerCRC
	{
		uint32 CRC;
		bool bHasAlignedCRC;
		bool bError;
	};

	TArray<FHandlerCRC> HandlerCRCs;
	uint32 NetConnectionCRC = 0;
#endif

	if (ReturnVal == EIncomingResult::Success)
	{
		FBitReader ProcessedPacketReader(DataView.GetMutableData(), CountBits);
		FIncomingPacketRef PacketRef = {ProcessedPacketReader, PacketView.Address, PacketView.Traits};

		FPacketAudit::CheckStage(TEXT("PostPacketHandler"), ProcessedPacketReader);

		if (State == UE::Handler::State::Uninitialized)
		{
			UpdateInitialState();
		}


		for (int32 i=HandlerComponents.Num() - 1; i>=0; --i)
		{
			HandlerComponent& CurComponent = *HandlerComponents[i];

#if !UE_BUILD_SHIPPING
			if (UNLIKELY(!!GPacketHandlerCRCDump))
			{
				if (ProcessedPacketReader.IsError())
				{
					HandlerCRCs.Add({0, false, true});
				}
				else if (ProcessedPacketReader.GetPosBits() == 0)
				{
					HandlerCRCs.Add({FCrc::MemCrc32(ProcessedPacketReader.GetData(), IntCastLog<int32, int64>(ProcessedPacketReader.GetNumBytes())), true, false});
				}
				else
				{
					HandlerCRCs.Add({0, false, false});
				}
			}
#endif

			if (CurComponent.IsActive() && !ProcessedPacketReader.IsError() && ProcessedPacketReader.GetBitsLeft() > 0)
			{
				// Realign the packet, so the packet data starts at position 0, if necessary
				if (ProcessedPacketReader.GetPosBits() != 0 && !CurComponent.CanReadUnaligned())
				{
					RealignPacket(ProcessedPacketReader);

#if !UE_BUILD_SHIPPING
					if (UNLIKELY(!!GPacketHandlerCRCDump))
					{
						FHandlerCRC& CurCRC = HandlerCRCs[HandlerCRCs.Num() - 1];

						CurCRC.CRC = FCrc::MemCrc32(ProcessedPacketReader.GetData(), IntCastLog<int32, int64>(ProcessedPacketReader.GetNumBytes()));
						CurCRC.bHasAlignedCRC = true;
					}
#endif
				}

				if (PacketView.Traits.bConnectionlessPacket)
				{
					CurComponent.IncomingConnectionless(PacketRef);
				}
				else
				{
					NET_LIGHTWEIGHT_TIME_GUARD_BEGIN(Incoming, GPacketHandlerTimeguardThresholdMS);

					CurComponent.Incoming(PacketRef);

					NET_LIGHTWEIGHT_TIME_GUARD_END(Incoming, CurComponent.GetName().ToString());
				}
			}
		}

		if (!ProcessedPacketReader.IsError())
		{
			ReplaceIncomingPacket(ProcessedPacketReader);

			if (IncomingPacket.GetBitsLeft() > 0)
			{
				FPacketAudit::CheckStage(TEXT("PrePacketHandler"), IncomingPacket, true);

#if !UE_BUILD_SHIPPING
				if (UNLIKELY(!!GPacketHandlerCRCDump))
				{
					NetConnectionCRC = FCrc::MemCrc32(IncomingPacket.GetData(), IntCastLog<int32, int64>(IncomingPacket.GetBytesLeft()));
				}
#endif
			}

			PacketView.DataView = {IncomingPacket.GetData(), (int32)IncomingPacket.GetBitsLeft(), ECountUnits::Bits};
		}
		else
		{
			PacketView.DataView = {nullptr, 0, ECountUnits::Bits};
			ReturnVal = EIncomingResult::Error;
		}
	}

#if !UE_BUILD_SHIPPING
	if (UNLIKELY(!!GPacketHandlerCRCDump))
	{
		TStringBuilder<2048> HandlerCRCStr;

		for (int32 i=HandlerCRCs.Num()-1; i>=0; i--)
		{
			FHandlerCRC& CurCRC = HandlerCRCs[i];

			HandlerCRCStr.Appendf(TEXT("%s%i: "), (i != HandlerCRCs.Num()-1 ? TEXT(", ") : TEXT("")), (HandlerCRCs.Num() - 1) - i);

			if (CurCRC.bError)
			{
				HandlerCRCStr << TEXT("Error");
			}
			else if (!CurCRC.bHasAlignedCRC)
			{
				HandlerCRCStr << TEXT("Unaligned");
			}
			else
			{
				HandlerCRCStr << FString::Printf(TEXT("%08X"), CurCRC.CRC);
			}
		}

		UE_LOGF(PacketHandlerLog, Log, "PacketHandler::Incoming: CRC Dump: NetConnection: %ls, Component: %ls, Socket: %08X",
				(ReturnVal == EIncomingResult::Success ? *FString::Printf(TEXT("%08X"), NetConnectionCRC) : TEXT("Error")),
				*HandlerCRCStr, SocketCRC);
	}
#endif

	return ReturnVal;
}

const ProcessedPacket PacketHandler::Outgoing_Internal(uint8* Packet, int32 CountBits, FOutPacketTraits& Traits, bool bConnectionless, const TSharedPtr<const FInternetAddr>& Address)
{
	SCOPE_CYCLE_COUNTER(Stat_PacketHandler_Outgoing_Internal);

	ProcessedPacket ReturnVal;

#if !UE_BUILD_SHIPPING
	uint32 NetConnectionCRC = 0;

	struct FHandlerCRC
	{
		uint32 CRC;
		bool bError;
	};

	TArray<FHandlerCRC> HandlerCRCs;
	
	if (UNLIKELY(!!GPacketHandlerCRCDump))
	{
		NetConnectionCRC = FCrc::MemCrc32(Packet, FMath::DivideAndRoundUp(CountBits, 8));
	}
#endif

	if (!bRawSend)
	{
		OutgoingPacket.Reset();

		if (State == UE::Handler::State::Uninitialized)
		{
			UpdateInitialState();
		}


		if (State == UE::Handler::State::Initialized)
		{
			OutgoingPacket.SerializeBits(Packet, CountBits);

			FPacketAudit::AddStage(TEXT("PrePacketHandler"), OutgoingPacket, true);

			for (int32 i=0; i<HandlerComponents.Num() && !OutgoingPacket.IsError(); ++i)
			{
				HandlerComponent& CurComponent = *HandlerComponents[i];

				if (CurComponent.IsActive())
				{
					if (OutgoingPacket.GetNumBits() <= CurComponent.MaxOutgoingBits)
					{
						if (bConnectionless)
						{
							CurComponent.OutgoingConnectionless(Address, OutgoingPacket, Traits);
						}
						else
						{
							NET_LIGHTWEIGHT_TIME_GUARD_BEGIN(Outgoing, GPacketHandlerTimeguardThresholdMS);

							CurComponent.Outgoing(OutgoingPacket, Traits);

							NET_LIGHTWEIGHT_TIME_GUARD_END(Outgoing, CurComponent.GetName().ToString());
						}
					}
					else
					{
						OutgoingPacket.SetError();

						UE_LOG(PacketHandlerLog, Error, TEXT("Packet exceeded HandlerComponents 'MaxOutgoingBits' value: %" INT64_FMT " vs %i"),
								OutgoingPacket.GetNumBits(), CurComponent.MaxOutgoingBits);

						break;
					}
				}

#if !UE_BUILD_SHIPPING
				if (UNLIKELY(!!GPacketHandlerCRCDump))
				{
					if (OutgoingPacket.IsError())
					{
						HandlerCRCs.Add({0, true});
					}
					else
					{
						HandlerCRCs.Add({FCrc::MemCrc32(OutgoingPacket.GetData(), IntCastLog<int32, int64>(OutgoingPacket.GetNumBytes())), false});
					}
				}
#endif
			}

			// Add a termination bit, the same as the UNetConnection code does, if appropriate
			if (HandlerComponents.Num() > 0 && OutgoingPacket.GetNumBits() > 0)
			{
				FPacketAudit::AddStage(TEXT("PostPacketHandler"), OutgoingPacket);

				OutgoingPacket.WriteBit(1);
			}

			if (!bConnectionless && ReliabilityComponent.IsValid() && OutgoingPacket.GetNumBits() > 0)
			{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
				// Let the reliability handler know about all processed packets, so it can record them for resending if needed
				ReliabilityComponent->QueuePacketForResending(OutgoingPacket.GetData(), IntCastLog<int32, int64>(OutgoingPacket.GetNumBits()), Traits);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
			}
		}
		// Buffer any packets being sent from game code until processors are initialized
		else if (State == UE::Handler::State::InitializingComponents && CountBits > 0)
		{
			if (bConnectionless)
			{
				BufferedConnectionlessPackets.Add(new BufferedPacket(Address, Packet, CountBits, Traits));
			}
			else
			{
				BufferedPackets.Add(new BufferedPacket(Packet, CountBits, Traits));
			}

			Packet = nullptr;
			CountBits = 0;
		}

		if (!OutgoingPacket.IsError())
		{
			ReturnVal = {OutgoingPacket.GetData(), (int32)OutgoingPacket.GetNumBits()};
		}
		else
		{
			ReturnVal = {nullptr, 0, true};
		}
	}
	else
	{
		ReturnVal = {Packet, CountBits};
	}

#if !UE_BUILD_SHIPPING
	if (UNLIKELY(!!GPacketHandlerCRCDump))
	{
		TStringBuilder<2048> HandlerCRCStr;

		for (int32 i=0; i<HandlerCRCs.Num(); i++)
		{
			FHandlerCRC& CurCRC = HandlerCRCs[i];

			HandlerCRCStr.Appendf(TEXT("%s%i: "), (i > 0 ? TEXT(", ") : TEXT("")), i);
			
			if (CurCRC.bError)
			{
				HandlerCRCStr << TEXT("Error");
			}
			else
			{
				HandlerCRCStr.Appendf(TEXT("%08X"), CurCRC.CRC);
			}
		}

		TStringBuilder<32> SocketCRCStr;

		if (ReturnVal.bError)
		{
			SocketCRCStr << TEXT("Error");
		}
		else
		{
			SocketCRCStr.Appendf(TEXT("%08X"), FCrc::MemCrc32(ReturnVal.Data, FMath::DivideAndRoundUp(ReturnVal.CountBits, 8)));
		}

		UE_LOGF(PacketHandlerLog, Log, "PacketHandler::Outgoing: CRC Dump: NetConnection: %08X, Component: %ls, Socket: %ls",
				NetConnectionCRC, *HandlerCRCStr, *SocketCRCStr);
	}
#endif

	return ReturnVal;
}

bool PacketHandler::CanRunHandlerComponentsOnParallelOutgoingConnections()
{
	bool bCanRunInParallel = true;
	for (int32 i = 0; i < HandlerComponents.Num(); ++i)
	{
		if (HandlerComponents[i])
		{
			HandlerComponent& CurComponent = *HandlerComponents[i];
			bCanRunInParallel &= CurComponent.SupportsParallelConnectionOutgoing();
		}
	}

	return bCanRunInParallel;
}

void PacketHandler::LogHandlerComponentList()
{
	for (int32 i = 0; i < HandlerComponents.Num(); ++i)
	{
		if (HandlerComponents[i])
		{
			HandlerComponent& CurComponent = *HandlerComponents[i];

			UE_LOGF(PacketHandlerLog, Log, "Handler: %ls SupportsParallel: %ls", *CurComponent.GetName().ToString(), CurComponent.SupportsParallelConnectionOutgoing() ? TEXT("TRUE") : TEXT("FALSE"));
		}
	}
}

void PacketHandler::ReplaceIncomingPacket(FBitReader& ReplacementPacket)
{
	if (ReplacementPacket.GetPosBits() == 0 || ReplacementPacket.GetBitsLeft() == 0)
	{
		IncomingPacket = MoveTemp(ReplacementPacket);
	}
	else
	{
		// @todo #JohnB: Make this directly adjust and write into IncomingPacket's buffer, instead of copying - very inefficient
		TArray<uint8> TempPacketData;
		TempPacketData.AddUninitialized(IntCastLog<int32, int64>(ReplacementPacket.GetBytesLeft()));
		TempPacketData[TempPacketData.Num()-1] = 0;

		int64 NewPacketSizeBits = ReplacementPacket.GetBitsLeft();

		ReplacementPacket.SerializeBits(TempPacketData.GetData(), NewPacketSizeBits);
		IncomingPacket.SetData(MoveTemp(TempPacketData), NewPacketSizeBits);
	}
}

void PacketHandler::RealignPacket(FBitReader& Packet)
{
	if (Packet.GetPosBits() != 0)
	{
		const int32 BitsLeft = IntCastLog<int32, int64>(Packet.GetBitsLeft());

		if (BitsLeft > 0)
		{
			// @todo #JohnB: Based on above - when you optimize above, optimize this too
			TArray<uint8> TempPacketData;
			TempPacketData.AddUninitialized(IntCastLog<int32, int64>(Packet.GetBytesLeft()));
			TempPacketData[TempPacketData.Num()-1] = 0;

			Packet.SerializeBits(TempPacketData.GetData(), BitsLeft);
			Packet.SetData(MoveTemp(TempPacketData), BitsLeft);
		}
	}
}

void PacketHandler::SendHandlerPacket(HandlerComponent* InComponent, FBitWriter& Writer, FOutPacketTraits& Traits)
{
	// @todo #JohnB: There is duplication between this function and others, it would be nice to reduce this.

	// Prevent any cases where a send happens before the handler is ready.
	check(State != UE::Handler::State::Uninitialized);

	if (LowLevelSendDel.IsBound())
	{
		bool bEncounteredComponent = false;

		for (int32 i=0; i<HandlerComponents.Num() && !Writer.IsError(); ++i)
		{
			HandlerComponent& CurComponent = *HandlerComponents[i];

			// Only process the packet through components coming after the specified one
			if (!bEncounteredComponent)
			{
				if (&CurComponent == InComponent)
				{
					bEncounteredComponent = true;
				}

				continue;
			}

			if (CurComponent.IsActive())
			{
				if (Writer.GetNumBits() <= CurComponent.MaxOutgoingBits)
				{
					CurComponent.Outgoing(Writer, Traits);
				}
				else
				{
					Writer.SetError();

					UE_LOG(PacketHandlerLog, Error, TEXT("Handler packet exceeded HandlerComponents 'MaxOutgoingBits' value: %" INT64_FMT " vs %i"),
							Writer.GetNumBits(), CurComponent.MaxOutgoingBits);

					break;
				}
			}
		}

		if (!Writer.IsError() && Writer.GetNumBits() > 0)
		{
			FPacketAudit::AddStage(TEXT("PostPacketHandler"), Writer);

			// Add a termination bit, the same as the UNetConnection code does, if appropriate
			Writer.WriteBit(1);

			if (ReliabilityComponent.IsValid())
			{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
				// Let the reliability handler know about all processed packets, so it can record them for resending if needed
				ReliabilityComponent->QueueHandlerPacketForResending(InComponent, Writer.GetData(), IntCastLog<int32, int64>(Writer.GetNumBits()), Traits);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
			}

			// Now finish off with a raw send (as we don't want to go through the PacketHandler chain again)
			bool bOldRawSend = bRawSend;

			bRawSend = true;

			LowLevelSendDel.ExecuteIfBound(Writer.GetData(), IntCastLog<int32, int64>(Writer.GetNumBits()), Traits);

			bRawSend = bOldRawSend;
		}
	}
	else
	{
		LowLevelFatalError(TEXT("Called SendHandlerPacket when no LowLevelSend delegate is bound"));
	}
}

void PacketHandler::SetState(UE::Handler::State InState)
{
	if (InState == State)
	{
		LowLevelFatalError(TEXT("Set new Packet Processor State to the state it is currently in."));
	} 
	else
	{
		State = InState;
	}
}

void PacketHandler::UpdateInitialState()
{
	if (State == UE::Handler::State::Uninitialized)
	{
		if (HandlerComponents.Num() > 0)
		{
			InitializeComponents();
		} 
		else
		{
			HandlerInitialized();
		}
	}
}

void PacketHandler::HandlerInitialized()
{
	// Quickly verify that, if reliability is required, that it is enabled
	if (!ReliabilityComponent.IsValid())
	{
		for(const TSharedPtr<HandlerComponent>& Component : HandlerComponents)
		{
			if (Component.IsValid() && Component->RequiresReliability())
			{
				// Don't allow this to be missed in shipping - but allow it during development,
				// as this is valid when developing new HandlerComponent's
#if UE_BUILD_SHIPPING
				UE_LOGF(PacketHandlerLog, Fatal, "Some HandlerComponents require bEnableReliability!!!");
#else
				UE_LOGF(PacketHandlerLog, Warning, "Some HandlerComponents require bEnableReliability!!!");
#endif

				break;
			}
		}
	}

	// If any buffered packets, add to queue
	for (int32 i=0; i<BufferedPackets.Num(); ++i)
	{
		QueuedPackets.Enqueue(BufferedPackets[i]);
		BufferedPackets[i] = nullptr;
	}

	BufferedPackets.Empty();

	for (int32 i=0; i<BufferedConnectionlessPackets.Num(); ++i)
	{
		QueuedConnectionlessPackets.Enqueue(BufferedConnectionlessPackets[i]);
		BufferedConnectionlessPackets[i] = nullptr;
	}

	BufferedConnectionlessPackets.Empty();

	SetState(UE::Handler::State::Initialized);

	if (bBeganHandshaking)
	{
		HandshakeCompleteDel.ExecuteIfBound();
	}
}

void PacketHandler::HandlerComponentInitialized(HandlerComponent* InComponent)
{
	// Check if all handlers are initialized
	if (State != UE::Handler::State::Initialized)
	{
		bool bAllInitialized = true;
		bool bEncounteredComponent = false;
		bool bPassedHandshakeNotify = false;

		for (int32 i=HandlerComponents.Num() - 1; i>=0; --i)
		{
			HandlerComponent& CurComponent = *HandlerComponents[i];

			if (!CurComponent.IsInitialized())
			{
				bAllInitialized = false;
			}

			if (bEncounteredComponent)
			{
				// If the initialized component required a handshake, pass on notification to the next handshaking component
				// (components closer to the Socket, perform their handshake first)
				if (bBeganHandshaking && !CurComponent.IsInitialized() && InComponent->RequiresHandshake() && !bPassedHandshakeNotify &&
						CurComponent.RequiresHandshake())
				{
					CurComponent.NotifyHandshakeBegin();
					bPassedHandshakeNotify = true;
				}
			}
			else
			{
				bEncounteredComponent = &CurComponent == InComponent;
			}
		}

		if (bAllInitialized)
		{
			HandlerInitialized();
		}
	}
}

bool PacketHandler::DoesAnyProfileHaveComponent(const FString& InComponentName)
{
	TArray<FString> ProfileSectionNames;
	if (GConfig->GetPerObjectConfigSections(GEngineIni, TEXT("PacketHandlerProfileConfig"), ProfileSectionNames))
	{
		for (const FString& CurProfileSection : ProfileSectionNames)
		{
			FName CurNetDriver(*CurProfileSection.Left(CurProfileSection.Find(TEXT(" "))));
			if (DoesProfileHaveComponent(CurNetDriver, InComponentName))
			{
				return true;
			}
		}
	}

	return false;
}

bool PacketHandler::DoesProfileHaveComponent(const FName InNetDriverName, const FString& InComponentName)
{
	TArray<FString> Components;
	FString DriverProfileCategory = FString::Printf(TEXT("%s PacketHandlerProfileConfig"), *InNetDriverName.GetPlainNameString());
	GConfig->GetArray(*DriverProfileCategory, TEXT("Components"), Components, GEngineIni);
	
	for (const FString& Component : Components)
	{
		if (Component.Contains(InComponentName, ESearchCase::CaseSensitive))
		{
			return true;
		}
	}
	return false;
}

BufferedPacket* PacketHandler::GetQueuedPacket()
{
	BufferedPacket* QueuedPacket = nullptr;

	QueuedPackets.Dequeue(QueuedPacket);

	return QueuedPacket;
}

BufferedPacket* PacketHandler::GetQueuedRawPacket()
{
	BufferedPacket* QueuedPacket = nullptr;

	QueuedRawPackets.Dequeue(QueuedPacket);

	return QueuedPacket;
}

BufferedPacket* PacketHandler::GetQueuedConnectionlessPacket()
{
	BufferedPacket* QueuedConnectionlessPacket = nullptr;

	QueuedConnectionlessPackets.Dequeue(QueuedConnectionlessPacket);

	return QueuedConnectionlessPacket;
}

int32 PacketHandler::GetTotalReservedPacketBits()
{
	int32 ReturnVal = 0;
	uint32 CurMaxOutgoingBits = MaxPacketBits;

	for (int32 i=HandlerComponents.Num()-1; i>=0; i--)
	{
		HandlerComponent* CurComponent = HandlerComponents[i].Get();
		int32 CurReservedBits = CurComponent->GetReservedPacketBits();

		// Specifying the reserved packet bits is mandatory, even if zero (as accidentally forgetting, leads to hard to trace issues).
		if (CurReservedBits == -1)
		{
			LowLevelFatalError(TEXT("Handler returned invalid 'GetReservedPacketBits' value."));
			continue;
		}


		// Set the maximum Outgoing packet size for the HandlerComponent
		CurComponent->MaxOutgoingBits = CurMaxOutgoingBits;
		CurMaxOutgoingBits -= CurReservedBits;

		ReturnVal += CurReservedBits;
	}


	// Reserve space for the termination bit
	if (HandlerComponents.Num() > 0)
	{
		ReturnVal++;
	}

	return ReturnVal;
}


/**
 * HandlerComponent
 */

HandlerComponent::HandlerComponent()
	: Handler(nullptr)
	, State(UE::Handler::Component::State::UnInitialized)
	, MaxOutgoingBits(0)
	, bRequiresHandshake(false)
	, bRequiresReliability(false)
	, bActive(false)
	, bInitialized(false)
{
}

HandlerComponent::HandlerComponent(FName InName)
	: Handler(nullptr)
	, State(UE::Handler::Component::State::UnInitialized)
	, MaxOutgoingBits(0)
	, bRequiresHandshake(false)
	, bRequiresReliability(false)
	, bActive(false)
	, bInitialized(false)
	, Name(InName)
{
}

HandlerComponent::~HandlerComponent() = default;


bool HandlerComponent::IsActive() const
{
	return bActive;
}

void HandlerComponent::SetActive(bool Active)
{
	bActive = Active;
}

void HandlerComponent::SetState(UE::Handler::Component::State InState)
{
	State = InState;
}

void HandlerComponent::Initialized()
{
	bInitialized = true;
	Handler->HandlerComponentInitialized(this);
}

bool HandlerComponent::IsInitialized() const
{
	return bInitialized;
}

/**
 * FPacketHandlerComponentModuleInterface
 */
void FPacketHandlerComponentModuleInterface::StartupModule()
{
	FPacketAudit::Init();
}

void FPacketHandlerComponentModuleInterface::ShutdownModule()
{
	FPacketAudit::Destruct();
}
