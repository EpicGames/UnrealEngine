// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PacketHandler.h"
#include "Containers/ArrayView.h"
#include "UObject/CoreNet.h"

/**
 * IEncryptionComponentInterface
 */
class FEncryptionComponent : public HandlerComponent
{
public:
	/**
	 * Constructor that forwards the name to the base HandlerComponent.
	 */
	explicit FEncryptionComponent(FName InName) : HandlerComponent(InName) {}

	/**
	 * Enable encryption. Future packets that are processed by this component will be encrypted. By default, encryption is disabled.
	 */
	virtual void EnableEncryption() = 0;

	/**
	 * Disable encryption. Future packets that are processed by this component will not be encrypted.
	 */
	virtual void DisableEncryption() = 0;

	/**
	 * Returns true if encryption is currently enabled.
	 */
	virtual bool IsEncryptionEnabled() const = 0;

	/**
	 * Sets the encryption key to be used by this component. The key must be the correct size for the encryption algorithm
	 * that the component implements. This should be called before EnableEncryption.
	 */
	UE_DEPRECATED(4.24, "Use SetEncryptionData instead")
	virtual void SetEncryptionKey(TArrayView<const uint8> Key)
	{
		FEncryptionData EncryptionData;
		EncryptionData.Key.Append(Key.GetData(), Key.Num());
		SetEncryptionData(EncryptionData);
	}

	/**
	 * Sets the encryption data to be used by this component. This should be called before EnableEncryption.
	 */
	virtual void SetEncryptionData(const FEncryptionData& EncryptionData) = 0;
};
