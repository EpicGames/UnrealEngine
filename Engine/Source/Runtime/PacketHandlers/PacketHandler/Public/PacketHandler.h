// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Queue.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "HAL/UnrealMemory.h"
#include "IPAddress.h"
#include "Logging/LogMacros.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "Modules/ModuleInterface.h"
#include "Net/Common/Packets/PacketTraits.h"
#include "Net/Common/Packets/PacketView.h"
#include "Serialization/Archive.h"
#include "Serialization/BitReader.h"
#include "Serialization/BitWriter.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"


PACKETHANDLER_API DECLARE_LOG_CATEGORY_EXTERN(PacketHandlerLog, Log, All);

class FInternetAddr;
class FDDoSDetection;
class FEncryptionComponent;
class FNetAnalyticsAggregator;
// Forward declarations
class HandlerComponent;
class IAnalyticsProvider;
class PacketHandler;
class ReliabilityHandlerComponent;

namespace UE
{
	namespace Net
	{
		class FNetConnectionFaultRecoveryBase;

#if !UE_BUILD_SHIPPING
		// bShouldDiscard=true will prevent the log warning from firing when calling NET_LIGHTWEIGHT_TIME_GUARD_END
		// If called in a parallel thread, this will only apply to the current thread of execution
		PACKETHANDLER_API void SetDiscardTimeguardMeasurement(const bool bShouldDiscard);
		PACKETHANDLER_API bool GetDiscardTimeguardMeasurement();
#endif
	}
}

/**
 * Delegates
 */

// Delegate for allowing access to LowLevelSend, without a dependency upon Engine
DECLARE_DELEGATE_ThreeParams(FPacketHandlerLowLevelSendTraits, void* /* Data */, int32 /* CountBits */, FOutPacketTraits& /* Traits */);

DECLARE_DELEGATE_ThreeParams(FPacketHandlerLowLevelSend, void* /* Data */, int32 /* CountBytes */, int32 /* CountBits */);

// Delegate for allowing adding of packet handlers without defining them in an ini file
DECLARE_DELEGATE_OneParam(FPacketHandlerAddComponentByNameDelegate, TArray<FString>& /* HandlerComponentNames */);
DECLARE_DELEGATE_OneParam(FPacketHandlerAddComponentDelegate, TArray<TSharedPtr<HandlerComponent>>& /* HandlerComponents*/ );

/**
 * Callback for notifying that a new HandlerComponent was added (to e.g. perform additional initialization)
 *
 * @param NewHandler	The newly added HandlerComponent
 */
DECLARE_DELEGATE_OneParam(FPacketHandlerNotifyAddHandler, TSharedPtr<HandlerComponent>& /*NewHandler*/);

/**
 * Callback for notifying higher-level code that handshaking has completed, and that packets are now ready to send without buffering
 */
DECLARE_DELEGATE(FPacketHandlerHandshakeComplete);


/**
 * Enums related to the PacketHandler
 */

namespace UE
{
	namespace Handler
	{
		/**
		* State of PacketHandler
		*/
		enum class State : uint8
		{
			Uninitialized,			// PacketHandler is uninitialized
			InitializingComponents,	// PacketHandler is initializing HandlerComponents
			Initialized				// PacketHandler and all HandlerComponents (if any) are initialized
		};
	
		/**
		* Mode of Packet Handler
		*/
		enum class Mode : uint8
		{
			Client,					// Clientside PacketHandler
			Server					// Serverside PacketHandler
		};
	
		namespace Component
		{
			/**
			* HandlerComponent State
			*/
			enum class State : uint8
			{
				UnInitialized,		// HandlerComponent not yet initialized
				InitializedOnLocal, // Initialized on local instance
				InitializeOnRemote, // Initialized on remote instance, not on local instance
				Initialized         // Initialized on both local and remote instances
			};
		}
	}
}

/**
 * The result of calling Incoming and Outgoing in the PacketHandler
 */
struct ProcessedPacket
{
	/** Pointer to the returned packet data */
	uint8* Data;

	/** Size of the returned packet data in bits */
	int32 CountBits;

	/** Whether or not there was an error processing the packet */
	bool bError;

public:
	ProcessedPacket()
		: Data()
		, CountBits(0)
		, bError(false)
	{
	}

	/**
	 * Base constructor
	 */
	ProcessedPacket(uint8* InData, int32 InCountBits, bool bInError=false)
		: Data(InData)
		, CountBits(InCountBits)
		, bError(bInError)
	{
	}
};

/**
 * PacketHandler will buffer packets, this struct is used to buffer such packets while handler components are initialized
 */
struct BufferedPacket
{
	/** Buffered packet data */
	uint8* Data;

	/** Size of buffered packet in bits */
	uint32 CountBits;

	/** Traits applied to the packet, if applicable */
	FOutPacketTraits Traits;

	/** Used by ReliabilityHandlerComponent, to mark a packet for resending */
	double ResendTime;

	/** Used by ReliabilityHandlerComponent, to track packet id's */
	uint32 Id;

	/** For connectionless packets, the address to send to */
	TSharedPtr<const FInternetAddr> Address;

	/** If buffering a packet through 'SendHandlerPacket', track the originating component */
	HandlerComponent* FromComponent;

private:
	/**
	 * Base constructor
	 */
	BufferedPacket()
		: Data(nullptr)
		, CountBits(0)
		, Traits()
		, ResendTime(0.0)
		, Id(0)
		, Address(nullptr)
		, FromComponent(nullptr)
	{
	}

public:
	BufferedPacket(uint8* InCopyData, uint32 InCountBits, FOutPacketTraits& InTraits, double InResendTime=0.0, uint32 InId=0)
		: CountBits(InCountBits)
		, Traits(InTraits)
		, ResendTime(InResendTime)
		, Id(InId)
		, Address(nullptr)
		, FromComponent(nullptr)
	{
		check(InCopyData != nullptr);

		Data = new uint8[FMath::DivideAndRoundUp(InCountBits, 8u)];
		FMemory::Memcpy(Data, InCopyData, FMath::DivideAndRoundUp(InCountBits, 8u));
	}

	BufferedPacket(TSharedPtr<const FInternetAddr> InAddress, uint8* InCopyData, uint32 InCountBits, FOutPacketTraits& InTraits, double InResendTime=0.0, uint32 InId=0)
		: BufferedPacket(InCopyData, InCountBits, InTraits, InResendTime, InId)
	{
		Address = InAddress;
	}

	/**
	 * Base destructor
	 */
	PACKETHANDLER_API ~BufferedPacket();

	void CountBytes(FArchive& Ar) const
	{
		Ar.CountBytes(sizeof(*this), sizeof(*this));
		Ar.CountBytes(FMath::DivideAndRoundUp(CountBits, 8u), FMath::DivideAndRoundUp(CountBits, 8u));
	}
};

/**
 * Parameters for HandlerComponent Incoming/IncomingConnectionless functions
 */
struct FIncomingPacketRef
{
	/** Incoming packet data being processed */
	FBitReader&								Packet;

	/** For IncomingConnectionless, the address of the received packet */
	const TSharedPtr<const FInternetAddr>&	Address;

	/** Metadata and flags for the received packet, indicating what it contains and how to process it */
	FInPacketTraits&						Traits;
};


/** Result for PacketHandler.Incoming */
enum class EIncomingResult : uint8
{
	Success,
	Error
};

/**
 * This class maintains an array of all PacketHandler Components and forwards incoming and outgoing packets the each component
 */
class PacketHandler
{
public:
	/**
	 * Base constructor
	 *
	 * @param InDDoS			Reference to the owning net drivers DDoS detection handler
	 */
	PACKETHANDLER_API PacketHandler(FDDoSDetection* InDDoS=nullptr);

	/**
	 * Handles initialization of manager
	 *
	 * @param Mode					The mode the manager should be initialized in
	 * @param InMaxPacketBits		The maximum supported packet size
	 * @param bConnectionlessOnly	Whether or not this is a connectionless-only manager (ignores .ini components)
	 * @param InProvider			The analytics provider
	 * @param InDDoS				Reference to the owning net drivers DDoS detection handler
	 * @param InDriverProfile		The PacketHandler configuration profile to use
	 */
	PACKETHANDLER_API void Initialize(UE::Handler::Mode Mode, uint32 InMaxPacketBits, bool bConnectionlessOnly=false,
					TSharedPtr<class IAnalyticsProvider> InProvider=nullptr, FDDoSDetection* InDDoS=nullptr, FName InDriverProfile=NAME_None);

	/**
	 * Used for external initialization of delegates
	 *
	 * @param InLowLevelSendDel		The delegate the PacketHandler should use for triggering packet sends
	 * @param InAddHandlerDel		Callback for notifying of new HandlerComponent's
	 */
	PACKETHANDLER_API void InitializeDelegates(FPacketHandlerLowLevelSendTraits InLowLevelSendDel,
								FPacketHandlerNotifyAddHandler InAddHandlerDel=FPacketHandlerNotifyAddHandler());

	/**
	 * Initializes a reference to the NetConnection fault recovery interface (does not require Engine dependency)
	 *
	 * @param InFaultRecovery	A reference to the fault recovery interface
	 */
	PACKETHANDLER_API void InitFaultRecovery(UE::Net::FNetConnectionFaultRecoveryBase* InFaultRecovery);

	/**
	 * Notification that the NetDriver analytics provider has been updated (NOT called on first initialization)
	 * NOTE: Can also mean disabled, e.g. during hotfix
	 *
	 * @param InProvider		The analytics provider
	 * @param InAggregator		The net analytics aggregator
	 */
	PACKETHANDLER_API void NotifyAnalyticsProvider(TSharedPtr<IAnalyticsProvider> InProvider, TSharedPtr<FNetAnalyticsAggregator> InAggregator);

	/**
	 * Triggers initialization of HandlerComponents.
	 */
	PACKETHANDLER_API void InitializeComponents();


	/**
	 * Triggered by the higher level netcode, to begin any required HandlerComponent handshakes
	 */
	PACKETHANDLER_API void BeginHandshaking(FPacketHandlerHandshakeComplete InHandshakeDel=FPacketHandlerHandshakeComplete());


	PACKETHANDLER_API void Tick(float DeltaTime);

	/**
	 * Adds a HandlerComponent to the pipeline, prior to initialization (none can be added after initialization)
	 *
	 * @param NewHandler		The HandlerComponent to add
	 * @param bDeferInitialize	Whether or not to defer triggering Initialize (for batch-adds - code calling this, triggers it instead)
	 */
	PACKETHANDLER_API void AddHandler(TSharedPtr<HandlerComponent>& NewHandler, bool bDeferInitialize=false);

	/**
	 * As above, but initializes from a string specifying the component module, and (optionally) additional options
	 *
	 * @param ComponentStr		The handler component to load
	 * @param bDeferInitialize	Whether or not to defer triggering Initialize (for batch-adds - code calling this, triggers it instead)
	 */
	PACKETHANDLER_API TSharedPtr<HandlerComponent> AddHandler(const FString& ComponentStr, bool bDeferInitialize=false);


	// @todo #JohnB: Add runtime-calculated arrays for each packet pipeline type, to reduce redundant iterations,
	//				(there are 3x iterations now, 1 for each packet pipeline type), and to ignore inactive HandlerComponent's

	// @todo #JohnB: The reserved packet bits needs to be handled differently for the 'High' functions, as they use SendBuffer,
	//					which normally is reduced in size by reserved packet bits.


	/**
	 * @todo #JohnB: Work in progress, don't use yet.
	 *
	 * Processes incoming packets at the UNetConnection level, after uncapping the packet into an FBitReader.
	 *
	 * Use this for simple data additions to packets, and for maximum compatibility with other HandlerComponent's.
	 *
	 * @param Reader	The FBitReader for the incoming packet
	 */
	PACKETHANDLER_API void IncomingHigh(FBitReader& Reader);

	/**
	 * @todo #JohnB: Work in progress, don't use yet.
	 *
	 * Processes outgoing packets at the UNetConnection level, after game data is written, and just before the packet is capped.
	 *
	 * Use this for simple data additions to packets, and for maximum compatibility with other HandlerComponent's.
	 *
	 * @param Writer	The FBitWriter for the outgoing packet
	 */
	PACKETHANDLER_API void OutgoingHigh(FBitWriter& Writer);

	// @todo: Don't deprecate, until after the NetDriver refactor
	//UE_DEPRECATED(4.26, "Incoming now uses FReceivedPacketView.")
	inline const ProcessedPacket Incoming(uint8* Packet, int32 CountBytes)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		const TSharedPtr<const FInternetAddr> EmptyAddress = nullptr;
		return Incoming_Internal(Packet, CountBytes, false, EmptyAddress);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	 * Processes incoming packets at the PacketHandler level, before any UNetConnection processing takes place on the packet.
	 *
	 * Use this for more complex changes to packets, such as compression/encryption,
	 * but be aware that compatibility problems with other HandlerComponent's are more likely.
	 *
	 * @param PacketView	View of the packet being processed - PacketView.Data should be used to return the result
	 * @return				Returns Success/Failure
	 */
	inline EIncomingResult Incoming(FReceivedPacketView& PacketView)
	{
		return Incoming_Internal(PacketView);
	}

	/**
	 * Processes outgoing packets at the PacketHandler level, after all UNetConnection processing.
	 *
	 * Use this for more complex changes to packets, such as compression/encryption,
	 * but be aware that compatibility problems with other HandlerComponent's are more likely.
	 *
	 * @param Packet		The packet data to be processed
	 * @param CountBits		The size of the packet data in bits
	 * @param Traits		Traits for the packet, passed down from the NetConnection
	 * @return				Returns the final packet
	 */
	inline const ProcessedPacket Outgoing(uint8* Packet, int32 CountBits, FOutPacketTraits& Traits)
	{
		const TSharedPtr<const FInternetAddr> EmptyAddress = nullptr;
		return Outgoing_Internal(Packet, CountBits, Traits, false, EmptyAddress);
	}


	// @todo: Don't deprecate, until after the NetDriver refactor
	//UE_DEPRECATED(4.26, "IncomingConnectionless now uses FReceivedPacketView.")
	inline const ProcessedPacket IncomingConnectionless(const TSharedPtr<const FInternetAddr>& Address, uint8* Packet, int32 CountBytes)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return Incoming_Internal(Packet, CountBytes, true, Address);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	 * Processes incoming packets without a UNetConnection, in the same manner as 'Incoming' above
	 * IMPORTANT: Net drivers triggering this, should call 'UNetDriver::FlushHandler' shortly afterwards, to minimize packet buffering
	 * NOTE: Connectionless packets are unreliable.
	 *
	 * @param PacketView	View of the packet being processed - PacketView.Data should be used to return the result
	 * @return				Returns Success/Failure
	 */
	inline EIncomingResult IncomingConnectionless(FReceivedPacketView& PacketView)
	{
		PacketView.Traits.bConnectionlessPacket = true;

		return Incoming_Internal(PacketView);
	}

	/**
	 * Processes outgoing packets without a UNetConnection, in the same manner as 'Outgoing' above
	 * NOTE: Connectionless packets are unreliable.
	 *
	 * @param Address		The address the packet is being sent to
	 * @param Packet		The packet data to be processed
	 * @param CountBits		The size of the packet data in bits
	 * @param Traits		Traits for the packet, if applicable
	 * @return				Returns the final packet
	 */
	inline const ProcessedPacket OutgoingConnectionless(const TSharedPtr<const FInternetAddr>& Address, uint8* Packet, int32 CountBits, FOutPacketTraits& Traits)
	{
		return Outgoing_Internal(Packet, CountBits, Traits, true, Address);
	}

	/** Returns a pointer to the component set as the encryption handler, if any. */
	PACKETHANDLER_API TSharedPtr<FEncryptionComponent> GetEncryptionComponent();

	/** Returns a pointer to the first component in the HandlerComponents array with the specified name. */
	PACKETHANDLER_API TSharedPtr<HandlerComponent> GetComponentByName(FName ComponentName) const;

	PACKETHANDLER_API void CountBytes(FArchive& Ar) const;

protected:
	UE_DEPRECATED(4.26, "Incoming_Internal now uses FReceivedPacketView")
	const ProcessedPacket Incoming_Internal(uint8* Packet, int32 CountBytes, bool bConnectionless, const TSharedPtr<const FInternetAddr>& Address)
	{
		FReceivedPacketView PacketView;

		PacketView.DataView = {Packet, CountBytes, ECountUnits::Bytes};
		PacketView.Address = Address;
		PacketView.Traits.bConnectionlessPacket = bConnectionless;

		EIncomingResult Result = Incoming_Internal(PacketView);

		if (Result == EIncomingResult::Success)
		{
			return ProcessedPacket(PacketView.DataView.GetMutableData(), PacketView.DataView.NumBits());
		}
		else
		{
			return ProcessedPacket(nullptr, 0, true);
		}
	}

	/**
	 * Internal handling for Incoming/IncomingConnectionless
	 *
	 * @param PacketView	View of the packet being processed - PacketView.Data should be used to return the result
	 * @return				Returns Success/Failure
	 */
	PACKETHANDLER_API EIncomingResult Incoming_Internal(FReceivedPacketView& PacketView);

	/**
	 * Internal handling for Outgoing/OutgoingConnectionless
	 *
	 * @param Packet			The packet data to be processed
	 * @param CountBits			The size of the packet data in bits
	 * @param Traits			Traits for the packet, passed down from the NetConnection, if applicable
	 * @param bConnectionless	Whether or not this should be sent as a connectionless packet
	 * @param Address			The address the packet is being sent to
	 * @return					Returns the final packet
	 */
	PACKETHANDLER_API const ProcessedPacket Outgoing_Internal(uint8* Packet, int32 CountBits, FOutPacketTraits& Traits, bool bConnectionless, const TSharedPtr<const FInternetAddr>& Address);

public:

	/**
	 * Send a packet originating from a HandlerComponent - will process through the HandlerComponents chain,
	 * starting after the triggering component.
	 * NOTE: Requires that InitializeDelegates is called, with a valid LowLevelSend delegate.
	 *
	 * @param Component		The HandlerComponent sending the packet
	 * @param Writer		The packet being sent
	 * @param Traits		The traits applied to the packet, if applicable
	 */
	PACKETHANDLER_API void SendHandlerPacket(HandlerComponent* InComponent, FBitWriter& Writer, FOutPacketTraits& Traits);


	/**
	 * Triggered when a child HandlerComponent has been initialized
	 */
	PACKETHANDLER_API void HandlerComponentInitialized(HandlerComponent* InComponent);

	/**
	 * Queue's a packet to be sent when the handler is ticked (as a raw packet, since it's already been processed)
	 *
	 * @param PacketToQueue		The packet to be queued
	 */
	inline void QueuePacketForRawSending(BufferedPacket* PacketToQueue)
	{
		QueuedRawPackets.Enqueue(PacketToQueue);
	}

	/**
	 * Queue's a packet to be sent through 'SendHandlerPacket'
	 *
	 * @param PacketToQueue		The packet to be queued
	 */
	inline void QueueHandlerPacketForSending(BufferedPacket* PacketToQueue)
	{
		QueuedHandlerPackets.Enqueue(PacketToQueue);
	}

	/**
	 * Searches the PacketHandler profile configurations to find if a component is listed.
	 *
	 * @param InComponentName	The PacketHandler Component to search for
	 * @return if there is a profile that has the component included.
	 */
	static PACKETHANDLER_API bool DoesAnyProfileHaveComponent(const FString& InComponentName);

	/**
	 * Searches the PacketHandler profile configuration for the given netdriver to find if a component is listed.
	 *
	 * @param InNetDriverName	The name of the netdriver to search configuration for
	 * @param InComponentName	The component to search for
	 * @return if the component is listed in the profile configuration.
	 */
	static PACKETHANDLER_API bool DoesProfileHaveComponent(const FName InNetDriverName, const FString& InComponentName);

	/**
	 * Gets a packet from the buffered packet queue for sending
	 *
	 * @return		The packet to be sent, or nullptr if none are to be sent
	 */
	PACKETHANDLER_API BufferedPacket* GetQueuedPacket();

	/**
	* Gets a packet from the buffered packet queue for sending (as a raw packet)
	*
	* @return		The packet to be sent, or nullptr if none are to be sent
	*/
	PACKETHANDLER_API BufferedPacket* GetQueuedRawPacket();

	/**
	 * Gets a packet from the buffered connectionless packet queue for sending
	 *
	 * @return		The packet to be sent, or nullptr if none are to be sent
	 */
	PACKETHANDLER_API BufferedPacket* GetQueuedConnectionlessPacket();

	/**
	 * Gets the combined reserved packet/protocol bits from all handlers, for reserving space in the parent connections packets
	 *
	 * @return	The combined reserved packet/protocol bits
	 */
	PACKETHANDLER_API int32 GetTotalReservedPacketBits();

	/**
	 * Whether or not all the HandlerComponents can support parallel Outgoing Connections
	 */
	PACKETHANDLER_API bool CanRunHandlerComponentsOnParallelOutgoingConnections();

	/**
	 * Output names of HandlerComponents to PacketHandlerLog
	 */
	PACKETHANDLER_API void LogHandlerComponentList();

	/**
	 * Sets whether or not outgoing packets should bypass this handler - used when raw packet sends are necessary
	 * (such as for the stateless handshake)
	 *
	 * @param bInEnabled	Whether or not raw sends are enabled
	 */
	inline void SetRawSend(bool bInEnabled)
	{
		bRawSend = bInEnabled;
	}

	/**
	 * Whether or not raw packet sends are enabled
	 */
	inline bool GetRawSend()
	{
		return bRawSend;
	}

	/**
	 * Whether or not the packet handler is fully initialized, post-handshake etc.
	 */
	inline bool IsFullyInitialized()
	{
		return State == UE::Handler::State::Initialized;
	}

	/** Returns a pointer to the DDoS detection handler */
	FDDoSDetection* GetDDoS() const { return DDoS; }

	/** Returns the analytics provider */
	TSharedPtr<IAnalyticsProvider> GetProvider() const { return Provider; }

	/** Returns the analytics aggregator */
	TSharedPtr<FNetAnalyticsAggregator> GetAggregator() const { return Aggregator; }


private:
	/**
	 * Set state of handler
	 *
	 * @param InState	The new state for the handler
	 */
	PACKETHANDLER_API void SetState(UE::Handler::State InState);

	/**
	 * Called when net send/receive functions are triggered, when the handler is still uninitialized - to set a valid initial state
	 */
	PACKETHANDLER_API void UpdateInitialState();

	/**
	 * Called when handler is finished initializing
	 */
	PACKETHANDLER_API void HandlerInitialized();

	/**
	 * Replaces IncomingPacket with all unread data from ReplacementPacket
	 *
	 * @param ReplacementPacket		The packet whose unread data should replace IncomingPacket
	 */
	PACKETHANDLER_API void ReplaceIncomingPacket(FBitReader& ReplacementPacket);

	/**
	 * Takes a Packet whose position is not at bit 0, and shifts/aligns all packet data to place the current bit at position 0
	 *
	 * @param Packet	The packet to realign
	 */
	PACKETHANDLER_API void RealignPacket(FBitReader& Packet);


public:
	/** Mode of the handler, Client or Server */
	UE::Handler::Mode Mode;

	UE_DEPRECATED(5.6, "PacketHandler AddComponent delegates are deprecated, use [PacketHandlerComponents] section in .ini file")
	static PACKETHANDLER_API FPacketHandlerAddComponentByNameDelegate& GetAddComponentByNameDelegate();
	UE_DEPRECATED(5.6, "PacketHandler AddComponent delegates are deprecated, use [PacketHandlerComponents] section in .ini file")
	static PACKETHANDLER_API FPacketHandlerAddComponentDelegate& GetAddComponentDelegate();

private:

	/** State of the handler */
	UE::Handler::State State;

	/** Whether or not outgoing packets bypass the handler */
	uint8 bRawSend:1;

	/** Whether or not component handshaking has begun */
	uint8 bBeganHandshaking:1;

	/** Whether or not this PacketHandler handles connectionless (i.e. non-UNetConnection) data */
	uint8 bConnectionlessHandler:1;

	/** The maximum supported packet size (reflects UNetConnection::MaxPacket) */
	uint32 MaxPacketBits;

	/** Mirroring UNetDriver.DDoS*/
	FDDoSDetection* DDoS;

	/** Delegate used for triggering PacketHandler/HandlerComponent-sourced sends */
	FPacketHandlerLowLevelSendTraits LowLevelSendDel;

	/** Delegate used for notifying that handshaking has completed */
	FPacketHandlerHandshakeComplete HandshakeCompleteDel;

	/** Delegate used for notifying that a new HandlerComponent has been added */
	FPacketHandlerNotifyAddHandler AddHandlerDel;

	/** Used for packing outgoing packets */
	FBitWriter OutgoingPacket;

	/** Used for unpacking incoming packets */
	FBitReader IncomingPacket;

	/** The HandlerComponent pipeline, for processing incoming/outgoing packets */
	TArray<TSharedPtr<HandlerComponent>> HandlerComponents;

	/** A direct pointer to the component configured as the encryption component. Will also be present in the HandlerComponents array. */
	TSharedPtr<FEncryptionComponent> EncryptionComponent;

	/** Packets that are buffered while HandlerComponents are being initialized */
	TArray<BufferedPacket*> BufferedPackets;

	/** Packets that are queued to be sent when handler is ticked */
	TQueue<BufferedPacket*> QueuedPackets;

	/** Packets that are queued to be sent when handler is ticked (as a raw packet) */
	TQueue<BufferedPacket*> QueuedRawPackets;

	/** Packets that are queued to be sent through 'SendHandlerPacket' */
	TQueue<BufferedPacket*> QueuedHandlerPackets;

	/** Packets that are buffered while HandlerComponents are being initialized */
	TArray<BufferedPacket*> BufferedConnectionlessPackets;

	/** Packets that are queued to be sent when handler is ticked */
	TQueue<BufferedPacket*> QueuedConnectionlessPackets;

	/** Reliability Handler Component */
	TSharedPtr<ReliabilityHandlerComponent> ReliabilityComponent;

	/** The analytics provider */
	TSharedPtr<IAnalyticsProvider> Provider;

	/** The NetDriver level aggregator for the analytics provider */
	TSharedPtr<FNetAnalyticsAggregator> Aggregator;
};

/**
 * This class appends or modifies incoming and outgoing packets on a connection
 */
class HandlerComponent
{
	friend class PacketHandler;

public:
	/**
	 * Base constructor
	 */
	PACKETHANDLER_API HandlerComponent();

	/**
	 * Constructor that accepts a name
	 */
	PACKETHANDLER_API explicit HandlerComponent(FName InName);

	/**
	 * Base destructor
	 */
	PACKETHANDLER_API virtual ~HandlerComponent();

	/**
	 * Returns whether this handler is currently active
	 */
	PACKETHANDLER_API virtual bool IsActive() const;

	/**
	 * Return whether this handler is valid
	 */
	virtual bool IsValid() const = 0;

	/**
	 * Returns whether this handler is initialized
	 */
	PACKETHANDLER_API bool IsInitialized() const;

	/**
	* Returns whether this handler perform a network handshake during initialization
	*/
	bool RequiresHandshake() const
	{
		return bRequiresHandshake;
	}

	/**
	* Returns whether this handler perform a network handshake during initialization
	*/
	bool RequiresReliability() const
	{
		return bRequiresReliability;
	}
	
	// Delay deprecation for initial checkin
	//UE_DEPRECATED(5.0, "Use the version of 'HandlerComponent::Incoming' which takes 'FIncomingPacketRef' instead")
	virtual void Incoming(FBitReader& Packet)
	{
	}

	/**
	 * Handles incoming packets
	 *
	 * @param PacketRef		Reference of the packet being processed
	 */
	virtual void Incoming(FIncomingPacketRef PacketRef)
	{
		// By default, call the deprecated version unless overridden (derived classes should NOT call the parent Incoming function)
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Incoming(PacketRef.Packet);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	 * Handles any outgoing packets
	 *
	 * @param Packet	The packet to be handled
	 * @param Traits	Traits for the packet, passed down through the packet pipeline (likely from the NetConnection)
	 */
	virtual void Outgoing(FBitWriter& Packet, FOutPacketTraits& Traits)
	{
	}

	/**
	 * Handles incoming packets not associated with a UNetConnection
	 *
	 * @param PacketRef		Reference of the packet being processed
	 */
	virtual void IncomingConnectionless(FIncomingPacketRef PacketRef)
	{
	}

	/**
	 * Handles any outgoing packets not associated with a UNetConnection
	 *
	 * @param Address	The address the packet is being sent to
	 * @param Packet	The packet to be handled
	 * @param Traits	Traits for the packet, passed down through the packet pipeline (if applicable)
	 */
	virtual void OutgoingConnectionless(const TSharedPtr<const FInternetAddr>& Address, FBitWriter& Packet, FOutPacketTraits& Traits)
	{
	}


	/**
	 * Whether or not the Incoming/IncomingConnectionless implementations, support reading Packets that aren't aligned at bit position 0
	 * (i.e. whether or not this handler supports bit-level, rather than byte-level, reads)
	 *
	 * @return	Whether or not the above is supported
	 */
	virtual bool CanReadUnaligned() const
	{
		return false;
	}

	/**
	 * Whether or not this HandlerComponent supports running its Outgoing logic on multiple separate connections simultaneously.
	 * Any accesses on a per-connection basis are safe, but writing to Global/NetDriver properties are not
	 *
	 * @return	Whether or not the above is supported
	 */
	virtual bool SupportsParallelConnectionOutgoing() const
	{
		return false;
	}

	/**
	 * Initialization functionality should be placed here
	 */
	virtual void Initialize() = 0;

	/**
	 * Initializes a reference to the NetConnection fault recovery interface (does not require Engine dependency)
	 *
	 * @param InFaultRecovery	A reference to the fault recovery interface
	 */
	virtual void InitFaultRecovery(UE::Net::FNetConnectionFaultRecoveryBase* InFaultRecovery)
	{
	}

	/**
	 * Notification to this component that it is ready to begin handshaking
	 */
	virtual void NotifyHandshakeBegin()
	{
	}

	/**
	 * Tick functionality should be placed here
	 */
	virtual void Tick(float DeltaTime) {}

	/**
	 * Sets whether this handler is currently active
	 *
	 * @param Active	Whether or not the handled should be active
	 */
	PACKETHANDLER_API virtual void SetActive(bool Active);

	/**
	 * Returns the amount of reserved packet/protocol bits expected from this component.
	 *
	 * IMPORTANT: This MUST be accurate, and should represent the worst-case number of reserved bits expected from the component.
	 *				If this is inaccurate, packets will randomly fail to send, in rare cases which are extremely hard to trace.
	 *
	 * @return	The worst-case reserved packet bits for the component
	 */
	virtual int32 GetReservedPacketBits() const = 0;

	/** Returns the name of this component. */
	FName GetName() const { return Name; }

	/**
	 * Notification that the analytics provider has been updated
	 * NOTE: Can also mean disabled, e.g. during hotfix
	 */
	virtual void NotifyAnalyticsProvider() {}

	PACKETHANDLER_API virtual void CountBytes(FArchive& Ar) const;

protected:
	/**
	 * Sets the state of the handler
	 *
	 * @param State		The new state for the handler
	 */
	PACKETHANDLER_API void SetState(UE::Handler::Component::State State);

	/**
	 * Should be called when the handler is fully initialized on both remote and local
	 */
	PACKETHANDLER_API void Initialized();


public:
	/** The manager of the handler, set in initialization */
	PacketHandler* Handler; 

protected:
	/** The state of this handler */
	UE::Handler::Component::State State;

	/** Maximum number of Outgoing packet bits supported (automatically calculated to factor in other HandlerComponent reserved bits) */
	uint32 MaxOutgoingBits;

	/** Whether this handler has to perform a network handshake during initialization (requires waiting on other HandlerComponent's) */
	bool bRequiresHandshake;

	/** Whether this handler depends upon the ReliabilityHandlerComponent being enabled */
	bool bRequiresReliability;

private:
	/** Whether this handler is active, which dictates whether it will receive incoming and outgoing packets. */
	bool bActive;

	/** Whether this handler is fully initialized on both remote and local */
	bool bInitialized;

	/* The name of this component */
	FName Name;
};

/**
 * PacketHandler Module Interface
 */
class FPacketHandlerComponentModuleInterface : public IModuleInterface
{
public:
	/* Creates an instance of this component */
	virtual TSharedPtr<HandlerComponent> CreateComponentInstance(FString& Options)
	{
		return nullptr;
	}

	PACKETHANDLER_API virtual void StartupModule() override;

	PACKETHANDLER_API virtual void ShutdownModule() override;
};
