// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReliabilityHandlerComponent.h"
#include "Modules/ModuleManager.h"
#include "Net/Core/Misc/PacketAudit.h"
#include "HAL/PlatformTime.h"

IMPLEMENT_MODULE(FReliabilityHandlerComponentModuleInterface, ReliabilityHandlerComponent);

// RELIABILITY
ReliabilityHandlerComponent::ReliabilityHandlerComponent()
: HandlerComponent(FName("ReliabilityHandlerComponent"))
, LocalPacketID(1)
, LocalPacketIDACKED(0)
, RemotePacketID(0)
, RemotePacketIDACKED(0)
, ResendResolutionTime(0.1)
, LastResendTime(0.0)
{
}

void ReliabilityHandlerComponent::CountBytes(FArchive& Ar) const
{
	HandlerComponent::CountBytes(Ar);

	const SIZE_T SizeOfThis = sizeof(*this) - sizeof(HandlerComponent);
	Ar.CountBytes(SizeOfThis, SizeOfThis);

	BufferedPackets.CountBytes(Ar);
	for (BufferedPacket const * const LocalPacket : BufferedPackets)
	{
		if (LocalPacket)
		{
			LocalPacket->CountBytes(Ar);
		}
	}
}

void ReliabilityHandlerComponent::Initialize()
{
	SetActive(true);
	Initialized();
	State = UE::Handler::Component::State::Initialized;
}

bool ReliabilityHandlerComponent::IsValid() const
{
	return true;
}

void ReliabilityHandlerComponent::Outgoing(FBitWriter& Packet, FOutPacketTraits& Traits)
{
	switch (State)
	{
		case UE::Handler::Component::State::Initialized:
		{
			check(IsActive() && IsValid());

			FBitWriter Local;
			Local.AllowAppend(true);
			Local.SetAllowResize(true);

			check(Packet.GetNumBytes() > 0);

			Local.SerializeIntPacked(RemotePacketID);
			Local.SerializeIntPacked(LocalPacketID);
			Local.SerializeBits(Packet.GetData(), Packet.GetNumBits());

			Packet = Local;

			FPacketAudit::AddStage(TEXT("PostReliability"), Packet);

			break;
		}
		default:
		{
			break;
		}
	}
}

void ReliabilityHandlerComponent::Incoming(FBitReader& Packet)
{
	switch (State)
	{
		case UE::Handler::Component::State::Initialized:
		{
			if (IsActive() && IsValid())
			{
				FPacketAudit::CheckStage(TEXT("PostReliability"), Packet);

				// Read ACK
				uint32 IncomingLocalPacketIDACK;
				Packet.SerializeIntPacked(IncomingLocalPacketIDACK);

				// Read Remote ID
				uint32 IncomingRemotePacketID;
				Packet.SerializeIntPacked(IncomingRemotePacketID);

				// Out of sequence or duplicate packet, ignore
				if (RemotePacketID + 1 != IncomingRemotePacketID)
				{
					Packet.SetData(nullptr, 0);
					return;
				}

				// Set latest ID
				RemotePacketID = IncomingRemotePacketID;

				check(IncomingLocalPacketIDACK >= LocalPacketIDACKED);

				// We don't record the latest ACK unless this packet is in-order, since we can't trust the ACK without further modifications
				LocalPacketIDACKED = IncomingLocalPacketIDACK;

				// Do not realign the remaining packet here, let the PacketHandler do that.
				// Previous code from here, had a bug that added an extra byte in some circumstances.
			}
			break;
		}
		default:
		{
			break;
		}
	}
}

void ReliabilityHandlerComponent::Tick(float DeltaTime)
{
	const double CurrentTime = FPlatformTime::Seconds();

	if (CurrentTime - LastResendTime < ResendResolutionTime)
	{
		return;
	}

	LastResendTime = CurrentTime;

	// Resend UNACKED packets
	// We resend all packets just to make sure
	// This is very inefficient and wastes bandwidth, we will want to implement NAK version at some point
	for (int i = 0; i < BufferedPackets.Num(); i++)
	{
		BufferedPacket* Packet = BufferedPackets[i];

		check(Packet->Id >= 1);

		if (LocalPacketIDACKED < Packet->Id)
		{
			// Send this as a raw packet, since it's already been processed
			Handler->QueuePacketForRawSending(Packet);
		}
		else
		{
			// This packet was ACK'd, we can remove
			check(i == 0);
			BufferedPackets.RemoveAt(i);
			i--;
			delete Packet;
		}
	}
}

void ReliabilityHandlerComponent::QueuePacketForResending(uint8* Packet, int32 CountBits, FOutPacketTraits& Traits)
{
	BufferedPackets.Add(new BufferedPacket(Packet, CountBits, Traits, FPlatformTime::Seconds() + ResendResolutionTime, LocalPacketID++));
}

int32 ReliabilityHandlerComponent::GetReservedPacketBits() const
{
	return 64;
}

// MODULE INTERFACE
TSharedPtr<HandlerComponent> FReliabilityHandlerComponentModuleInterface::CreateComponentInstance(FString& Options)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return MakeShareable(new ReliabilityHandlerComponent);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}
