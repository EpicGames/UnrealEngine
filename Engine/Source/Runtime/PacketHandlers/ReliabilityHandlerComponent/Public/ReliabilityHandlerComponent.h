// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Queue.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Net/Common/Packets/PacketTraits.h"
#include "PacketHandler.h"
#include "Templates/SharedPointer.h"

class FArchive;
class FString;
struct FBitReader;
struct FBitWriter;


class UE_DEPRECATED(5.3, "This handler is not supported. For fully-reliable data, use reliable RPCs or a separate connection with a reliable protocol.")
ReliabilityHandlerComponent : public HandlerComponent
{
public:
	/* Initializes default data */
	RELIABILITYHANDLERCOMPONENT_API ReliabilityHandlerComponent();

	virtual void CountBytes(FArchive& Ar) const;

	virtual void Initialize() override;

	virtual bool IsValid() const override;

	virtual void Tick(float DeltaTime) override;

	virtual void Incoming(FBitReader& Packet) override;
	virtual void Outgoing(FBitWriter& Packet, FOutPacketTraits& Traits) override;

	/* Queues a packet for resending */
	RELIABILITYHANDLERCOMPONENT_API void QueuePacketForResending(uint8* Packet, int32 CountBits, FOutPacketTraits& Traits);

	UE_DEPRECATED(4.21, "Use the PacketTraits version for sending packets with additional flags and options")
	inline void QueueHandlerPacketForResending(HandlerComponent* InComponent, uint8* Packet, int32 CountBits)
	{
		FOutPacketTraits EmptyTraits;
		QueueHandlerPacketForResending(InComponent, Packet, CountBits, EmptyTraits);
	}

	/**
	 * Queues a packet sent through SendHandlerPacket, for resending
	 *
	 * @param InComponent	The HandlerComponent the packet originated from
	 * @param Packet		The packet to be queued
	 * @param CountBits		The number of bits in the packet
	 */
	inline void QueueHandlerPacketForResending(HandlerComponent* InComponent, uint8* Packet, int32 CountBits, FOutPacketTraits& Traits)
	{
		QueuePacketForResending(Packet, CountBits, Traits);

		BufferedPackets[BufferedPackets.Num()-1]->FromComponent = InComponent;
	}

	virtual int32 GetReservedPacketBits() const override;

protected:
	/* Buffered Packets in case they need to be resent */
	TArray<BufferedPacket*> BufferedPackets;

	/* Latest Packet ID */
	uint32 LocalPacketID;

	/* Latest Packet ID that was ACKED */
	uint32 LocalPacketIDACKED;

	/* Latest Remote Packet ID */
	uint32 RemotePacketID;

	/* Latest Remote Packet ID that was ACKED */
	uint32 RemotePacketIDACKED;

	/* How long to wait before resending an UNACKED packet */
	double ResendResolutionTime;

	/* Last time we resent UNACKED packets */
	double LastResendTime;
};

/* Reliability Module Interface */
class FReliabilityHandlerComponentModuleInterface : public FPacketHandlerComponentModuleInterface
{
public:
	virtual TSharedPtr<HandlerComponent> CreateComponentInstance(FString& Options) override;
};
