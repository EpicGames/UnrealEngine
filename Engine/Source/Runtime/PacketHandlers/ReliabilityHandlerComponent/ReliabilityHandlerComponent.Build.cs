// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class ReliabilityHandlerComponent : ModuleRules
{
    public ReliabilityHandlerComponent(ReadOnlyTargetRules Target) : base(Target)
    {
        ShortName = "ReliableHComp";
        PublicDependencyModuleNames.AddRange(
            new string[] {
				"Core",
				"NetCore",
                "PacketHandler"
            }
        );
    }
}
